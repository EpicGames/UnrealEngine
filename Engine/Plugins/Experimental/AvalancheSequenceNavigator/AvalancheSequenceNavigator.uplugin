{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Motion Design Sequence Navigator Bridge",
	"Description": "Sequence Navigator Bridge for embedded Motion Design Sequences",
	"Category": "Animation",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"CanContainContent": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"EnabledByDefault": false,
	"SupportedTargetPlatforms": [
		"Win64",
		"Linux",
		"Mac"
	],
	"Modules": [
		{
			"Name": "AvalancheSequenceNavigator",
			"Type": "Editor",
			"LoadingPhase": "PostDefault"
		}
	],
	"Plugins": [
		{
			"Name": "Avalanche",
			"Enabled": true,
			"Optional": true
		},
		{
			"Name": "SequenceNavigator",
			"Enabled": true,
			"Optional": true
		}
	]
}
