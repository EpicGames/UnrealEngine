// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AvalancheSequenceNavigator : ModuleRules
{
	public AvalancheSequenceNavigator(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(new string[] {
			"Core"
		});

		PrivateDependencyModuleNames.AddRange(new string[] {
			"Avalanche",
			"AvalancheEditorCore",
			"AvalancheSequence",
			"AvalancheSequencer",
			"CoreUObject",
			"Engine",
			"MovieScene",
			"SequenceNavigator",
			"Sequencer",
			"SequencerCore",
			"Slate",
			"SlateCore",
			"ToolMenus",
			"UnrealEd"
		});
	}
}
