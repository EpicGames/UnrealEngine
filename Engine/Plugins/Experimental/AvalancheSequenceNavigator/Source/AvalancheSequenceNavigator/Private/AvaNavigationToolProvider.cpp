// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaNavigationToolProvider.h"
#include "ActorFactories/ActorFactory.h"
#include "AvaSequence.h"
#include "AvaSequenceActor.h"
#include "AvaSequenceNavigatorCommands.h"
#include "AvaSequencePlaybackObject.h"
#include "AvaSequencerUtils.h"
#include "Columns/AvaNavigationToolStatusColumn.h"
#include "Columns/NavigationToolColorColumn.h"
#include "Columns/NavigationToolColumnExtender.h"
#include "Columns/NavigationToolCommentColumn.h"
#include "Columns/NavigationToolDeactiveStateColumn.h"
#include "Columns/NavigationToolHBiasColumn.h"
#include "Columns/NavigationToolInTimeColumn.h"
#include "Columns/NavigationToolItemsColumn.h"
#include "Columns/NavigationToolLabelColumn.h"
#include "Columns/NavigationToolLengthColumn.h"
#include "Columns/NavigationToolLockColumn.h"
#include "Columns/NavigationToolMarkerVisibilityColumn.h"
#include "Columns/NavigationToolOutTimeColumn.h"
#include "Columns/NavigationToolPlayheadColumn.h"
#include "Columns/NavigationToolRevisionControlColumn.h"
#include "Columns/NavigationToolStartFrameOffsetColumn.h"
#include "Columns/NavigationToolTakeColumn.h"
#include "ContentBrowserModule.h"
#include "DragDrop/NavigationToolAvaSequenceDropHandler.h"
#include "Editor.h"
#include "Engine/Level.h"
#include "Filters/Filters/NavigationToolBuiltInFilterParams.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/Notifications/NotificationManager.h"
#include "IAvaSceneInterface.h"
#include "IAvaSequenceProvider.h"
#include "IAvaSequencer.h"
#include "IAvaSequencerProvider.h"
#include "IContentBrowserSingleton.h"
#include "INavigationTool.h"
#include "ISequencer.h"
#include "ISettingsModule.h"
#include "ISourceControlModule.h"
#include "ItemActions/NavigationToolAddItem.h"
#include "Items/NavigationToolAvaSequence.h"
#include "Items/NavigationToolItemParameters.h"
#include "NavigationToolSettings.h"
#include "ScopedTransaction.h"
#include "Settings/AvaSequencerSettings.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "AvaNavigationToolProvider"

namespace UE::AvaSequencer
{

const FName FAvaNavigationToolProvider::Identifier = TEXT("MotionDesign");

const FText FAvaNavigationToolProvider::MotionDesignColumnViewName = LOCTEXT("MotionDesignColumnViewName", "Motion Design");

const FName FAvaNavigationToolProvider::ToolbarSectionName = TEXT("MotionDesign");
const FName FAvaNavigationToolProvider::ContextMenuSectionName = TEXT("MotionDesignActions");

FAvaNavigationToolProvider::FAvaNavigationToolProvider(const TSharedRef<IAvaSequencer>& InAvaSequencer)
	: WeakAvaSequencer(InAvaSequencer)
	, ToolCommands(MakeShared<FUICommandList>())
{
}

FName FAvaNavigationToolProvider::GetIdentifier() const
{
	return Identifier;
}

TSet<TSubclassOf<UMovieSceneSequence>> FAvaNavigationToolProvider::GetSupportedSequenceClasses() const
{
	return { UAvaSequence::StaticClass() };
}

FText FAvaNavigationToolProvider::GetDefaultColumnView() const
{
	return MotionDesignColumnViewName;
}

FNavigationToolSaveState* FAvaNavigationToolProvider::GetSaveState(const UE::SequenceNavigator::INavigationTool& InTool) const
{
#if WITH_EDITOR
	if (IAvaSceneInterface* const AvaScene = GetSceneInterface(InTool))
	{
		return &AvaScene->GetNavigationToolSaveState();
	}
#endif
	return nullptr;
}

void FAvaNavigationToolProvider::SetSaveState(const UE::SequenceNavigator::INavigationTool& InTool, const FNavigationToolSaveState& InSaveState) const
{
#if WITH_EDITOR
	if (IAvaSceneInterface* const AvaScene = GetSceneInterface(InTool))
	{
		AvaScene->GetNavigationToolSaveState() = InSaveState;
	}
#endif
}

void FAvaNavigationToolProvider::BindCommands(const TSharedRef<FUICommandList>& InCommandList)
{
	const TSharedPtr<IAvaSequencer> AvaSequencer = WeakAvaSequencer.Pin();
	if (!AvaSequencer.IsValid())
	{
		return;
	}

	InCommandList->Append(ToolCommands);

	const FAvaSequenceNavigatorCommands& AvaSequenceNavigatorCommands = FAvaSequenceNavigatorCommands::Get();
	const FGenericCommands& GenericCommands = FGenericCommands::Get();

	ToolCommands->MapAction(GenericCommands.Duplicate
		, FExecuteAction::CreateSP(this, &FAvaNavigationToolProvider::DuplicateSelection)
		, FCanExecuteAction::CreateSP(this, &FAvaNavigationToolProvider::CanDuplicateSelection));

	ToolCommands->MapAction(GenericCommands.Delete
		, FExecuteAction::CreateSP(this, &FAvaNavigationToolProvider::DeleteSelection)
		, FCanExecuteAction::CreateSP(this, &FAvaNavigationToolProvider::CanDeleteSelection));

	ToolCommands->MapAction(GenericCommands.Rename
		, FExecuteAction::CreateSP(this, &FAvaNavigationToolProvider::RelabelSelection)
		, FCanExecuteAction::CreateSP(this, &FAvaNavigationToolProvider::CanRelabelSelection));

	ToolCommands->MapAction(AvaSequenceNavigatorCommands.AddNew
		, FExecuteAction::CreateSP(this, &FAvaNavigationToolProvider::AddSequenceToSelection)
		, FCanExecuteAction::CreateSP(this, &FAvaNavigationToolProvider::CanAddSequenceToSelection));

	ToolCommands->MapAction(AvaSequenceNavigatorCommands.PlaySelected
		, FExecuteAction::CreateSP(this, &FAvaNavigationToolProvider::PlaySelection)
		, FCanExecuteAction::CreateSP(this, &FAvaNavigationToolProvider::CanPlaySelection));

	ToolCommands->MapAction(AvaSequenceNavigatorCommands.ContinueSelected
		, FExecuteAction::CreateSP(this, &FAvaNavigationToolProvider::ContinueSelection)
		, FCanExecuteAction::CreateSP(this, &FAvaNavigationToolProvider::CanContinueSelection));

	ToolCommands->MapAction(AvaSequenceNavigatorCommands.StopSelected
		, FExecuteAction::CreateSP(this, &FAvaNavigationToolProvider::StopSelection)
		, FCanExecuteAction::CreateSP(this, &FAvaNavigationToolProvider::CanStopSelection));

	ToolCommands->MapAction(AvaSequenceNavigatorCommands.ExportSequence
		, FExecuteAction::CreateSP(this, &FAvaNavigationToolProvider::ExportSelection)
		, FCanExecuteAction::CreateSP(this, &FAvaNavigationToolProvider::CanExportSelection));

	ToolCommands->MapAction(AvaSequenceNavigatorCommands.SpawnSequencePlayer
		, FExecuteAction::CreateSP(this, &FAvaNavigationToolProvider::SpawnPlayersForSelection)
		, FCanExecuteAction::CreateSP(this, &FAvaNavigationToolProvider::CanSpawnPlayersForSelection));
}

void FAvaNavigationToolProvider::OnActivate()
{
	ExtendToolToolBar();
	ExtendToolItemContextMenu();

	DragDropInitializedDelegate = UE::SequenceNavigator::FNavigationToolExtender::OnItemDragDropOpInitialized()
		.AddSP(this, &FAvaNavigationToolProvider::ExtendToolItemDragDropOp);
}

void FAvaNavigationToolProvider::OnDeactivate()
{
	RemoveToolToolBarExtension();
	RemoveToolItemContextMenuExtension();

	UE::SequenceNavigator::FNavigationToolExtender::OnItemDragDropOpInitialized()
		.Remove(DragDropInitializedDelegate);
}

void FAvaNavigationToolProvider::OnExtendColumns(UE::SequenceNavigator::FNavigationToolColumnExtender& OutExtender)
{
	using namespace UE::SequenceNavigator;

	// Support built in columns
	OutExtender.AddColumn<FNavigationToolPlayheadColumn>();
	OutExtender.AddColumn<FNavigationToolDeactiveStateColumn>();
	OutExtender.AddColumn<FNavigationToolMarkerVisibilityColumn>();
	OutExtender.AddColumn<FNavigationToolLockColumn>();
	OutExtender.AddColumn<FNavigationToolColorColumn>();
	OutExtender.AddColumn<FNavigationToolLabelColumn>();
	OutExtender.AddColumn<FNavigationToolItemsColumn>();
	OutExtender.AddColumn<FNavigationToolInTimeColumn>();
	OutExtender.AddColumn<FNavigationToolOutTimeColumn>();
	OutExtender.AddColumn<FNavigationToolLengthColumn>();
	OutExtender.AddColumn<FNavigationToolHBiasColumn>();
	OutExtender.AddColumn<FNavigationToolStartFrameOffsetColumn>();
	OutExtender.AddColumn<FNavigationToolTakeColumn>();
	OutExtender.AddColumn<FNavigationToolCommentColumn>();

	ISourceControlModule& SourceControlModule = ISourceControlModule::Get();
	if (SourceControlModule.IsEnabled())
	{
		OutExtender.AddColumn<FNavigationToolRevisionControlColumn>();
	}

	// Add Motion Design specific columns
	OutExtender.AddColumn<FAvaNavigationToolStatusColumn>();

	FNavigationToolProvider::OnExtendColumns(OutExtender);
}

void FAvaNavigationToolProvider::OnExtendColumnViews(TSet<FNavigationToolColumnView>& OutColumnViews)
{
	using namespace UE::SequenceNavigator;

	FNavigationToolColumnView& MotionDesignColumnView = OutColumnViews
		.FindOrAdd(FNavigationToolColumnView(MotionDesignColumnViewName));

	MotionDesignColumnView.VisibleColumns =
		{
			FNavigationToolColorColumn::StaticColumnId(),
			FNavigationToolLabelColumn::StaticColumnId(),
			FNavigationToolItemsColumn::StaticColumnId(),
			FAvaNavigationToolStatusColumn::StaticColumnId()
		};

	FNavigationToolProvider::OnExtendColumnViews(OutColumnViews);
}

void FAvaNavigationToolProvider::OnExtendItemChildren(SequenceNavigator::INavigationTool& InTool
	, const SequenceNavigator::FNavigationToolViewModelPtr& InParentItem
	, TArray<SequenceNavigator::FNavigationToolViewModelWeakPtr>& OutWeakChildren
	, const bool bInRecursive)
{
	using namespace SequenceNavigator;

	FNavigationToolProvider::OnExtendItemChildren(InTool, InParentItem, OutWeakChildren, bInRecursive);

	// Only extending root item
	if (InParentItem->GetItemId() != FNavigationToolItemId::RootId)
	{
		return;
	}

	IAvaSequenceProvider* const SequenceProvider = GetSequenceProvider(InTool);
	if (!SequenceProvider)
	{
		return;
	}

	TArray<TWeakObjectPtr<UAvaSequence>> RootWeakSequences = SequenceProvider->GetRootSequences();

	// Remove invalid sequences and sort
	RootWeakSequences.RemoveAll([](const TWeakObjectPtr<UAvaSequence>& InRoot)
		{
			return !InRoot.IsValid();
		});
	RootWeakSequences.Sort([](const TWeakObjectPtr<UAvaSequence>& InA, const TWeakObjectPtr<UAvaSequence>& InB)
		{
			return InA->GetDisplayName().CompareTo(InB->GetDisplayName()) > 0;
		});

	const TSharedRef<FAvaNavigationToolProvider> ProviderRef = SharedThis(this);

	// Add child sequence items
	for (const TWeakObjectPtr<UAvaSequence>& RootSequenceWeak : RootWeakSequences)
	{
		if (UAvaSequence* const RootSequence = RootSequenceWeak.Get())
		{
			const FNavigationToolViewModelPtr NewItem = InTool.FindOrAdd<FNavigationToolAvaSequence>(ProviderRef
				, InParentItem, RootSequence);
			const FNavigationToolItemFlagGuard Guard(NewItem, ENavigationToolItemFlags::IgnorePendingKill);
			OutWeakChildren.Add(NewItem);
			if (bInRecursive)
			{
				NewItem->FindChildren(OutWeakChildren, bInRecursive);
			}
		}
	}
}

void FAvaNavigationToolProvider::OnExtendBuiltInFilters(TArray<FNavigationToolBuiltInFilterParams>& OutFilterParams)
{
	OutFilterParams.Add(FNavigationToolBuiltInFilterParams::CreateSequenceFilter());
	OutFilterParams.Add(FNavigationToolBuiltInFilterParams::CreateTrackFilter());
	OutFilterParams.Add(FNavigationToolBuiltInFilterParams::CreateBindingFilter());
	OutFilterParams.Add(FNavigationToolBuiltInFilterParams::CreateMarkerFilter());

	FNavigationToolProvider::OnExtendBuiltInFilters(OutFilterParams);
}

TSharedPtr<SequenceNavigator::INavigationTool> FAvaNavigationToolProvider::GetNavigationTool() const
{
	if (const TSharedPtr<IAvaSequencer> AvaSequencer = WeakAvaSequencer.Pin())
	{
		if (TSharedPtr<ISequencer> Sequencer = AvaSequencer->GetSequencerPtr())
		{
			return SequenceNavigator::FNavigationToolExtender::FindNavigationTool(Sequencer.ToSharedRef());
		}
	}
	return nullptr;
}

IAvaSceneInterface* FAvaNavigationToolProvider::GetSceneInterface(const UE::SequenceNavigator::INavigationTool& InTool) const
{
	if (const TSharedPtr<ISequencer> Sequencer = InTool.GetSequencer())
	{
		return FAvaSequencerUtils::GetSceneInterface(Sequencer.ToSharedRef());
	}
	return nullptr;
}

IAvaSequenceProvider* FAvaNavigationToolProvider::GetSequenceProvider(const UE::SequenceNavigator::INavigationTool& InTool) const
{
	if (const TSharedPtr<ISequencer> Sequencer = InTool.GetSequencer())
	{
		return FAvaSequencerUtils::GetSequenceProvider(Sequencer.ToSharedRef());
	}
	return nullptr;
}

const IAvaSequencerProvider* FAvaNavigationToolProvider::GetSequencerProvider() const
{
	if (const TSharedPtr<IAvaSequencer> AvaSequencer = WeakAvaSequencer.Pin())
	{
		return &AvaSequencer->GetProvider();
	}
	return nullptr;
}

IAvaSequencePlaybackObject* FAvaNavigationToolProvider::GetSequencerPlaybackObject() const
{
	if (const IAvaSequencerProvider* const SequencerProvider = GetSequencerProvider())
	{
		return SequencerProvider->GetPlaybackObject();
	}
	return nullptr;
}

bool FAvaNavigationToolProvider::CanEditOrPlaySelection(const int32 InMinNumSelected, const int32 InMaxNumSelected) const
{
	if (const IAvaSequencerProvider* const SequencerProvider = GetSequencerProvider())
	{
		if (SequencerProvider->CanEditOrPlaySequences())
		{
			const int32 NumSelected = GetSelectedSequenceItems().Num();
			return NumSelected >= InMinNumSelected
				&& (InMaxNumSelected == INDEX_NONE || NumSelected <= InMaxNumSelected);
		}
	}
	return false;
}

void FAvaNavigationToolProvider::ExtendToolToolBar()
{
	UToolMenu* const ToolMenu = UToolMenus::Get()->ExtendMenu(UE::SequenceNavigator::GetToolBarMenuName());
	if (!ToolMenu)
	{
		return;
	}

	FToolMenuSection& MotionDesignSection = ToolMenu->FindOrAddSection(ToolbarSectionName);

	const FAvaSequenceNavigatorCommands& AvaSequenceNavigatorCommands = FAvaSequenceNavigatorCommands::Get();
	const FName SequencerToolbarStyleName = TEXT("SequencerToolbar");

	FSlimHorizontalToolBarBuilder ToolBarBuilder(ToolCommands, FMultiBoxCustomization::None, nullptr, true);
	ToolBarBuilder.SetLabelVisibility(EVisibility::Collapsed);

	FToolMenuEntry SequenceAddNewEntry = FToolMenuEntry::InitToolBarButton(AvaSequenceNavigatorCommands.AddNew
		, TAttribute<FText>(), TAttribute<FText>()
		, FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("AnimationEditor.SetKey")));
	SequenceAddNewEntry.StyleNameOverride = SequencerToolbarStyleName;
	SequenceAddNewEntry.SetCommandList(ToolCommands);
	MotionDesignSection.AddEntry(SequenceAddNewEntry);

	FToolMenuEntry SequencePlaySelectedEntry = FToolMenuEntry::InitToolBarButton(AvaSequenceNavigatorCommands.PlaySelected
		, TAttribute<FText>(), TAttribute<FText>()
		, FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Icons.Toolbar.Play")));
	SequencePlaySelectedEntry.StyleNameOverride = SequencerToolbarStyleName;
	SequencePlaySelectedEntry.SetCommandList(ToolCommands);
	MotionDesignSection.AddEntry(SequencePlaySelectedEntry);

	FToolMenuEntry SequenceContinueSelectedEntry = FToolMenuEntry::InitToolBarButton(AvaSequenceNavigatorCommands.ContinueSelected
		, TAttribute<FText>(), TAttribute<FText>()
		, FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Icons.JumpToEvent")));
	SequenceContinueSelectedEntry.StyleNameOverride = SequencerToolbarStyleName;
	SequenceContinueSelectedEntry.SetCommandList(ToolCommands);
	MotionDesignSection.AddEntry(SequenceContinueSelectedEntry);

	FToolMenuEntry SequenceStopSelectedEntry = FToolMenuEntry::InitToolBarButton(AvaSequenceNavigatorCommands.StopSelected
		, TAttribute<FText>(), TAttribute<FText>()
		, FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Icons.Toolbar.Stop")));
	SequenceStopSelectedEntry.StyleNameOverride = SequencerToolbarStyleName;
	SequenceStopSelectedEntry.SetCommandList(ToolCommands);
	MotionDesignSection.AddEntry(SequenceStopSelectedEntry);
}

void FAvaNavigationToolProvider::ExtendToolItemContextMenu()
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	if (!ToolMenus)
	{
		return;
	}

	UToolMenu* const ExtendedMenu = ToolMenus->ExtendMenu(UE::SequenceNavigator::GetItemContextMenuName());
	if (!ExtendedMenu)
	{
		return;
	}

	const FAvaSequenceNavigatorCommands& AvaSequenceNavigatorCommands = FAvaSequenceNavigatorCommands::Get();

	FToolMenuSection& MotionDesignSection = ExtendedMenu->FindOrAddSection(ContextMenuSectionName
		, LOCTEXT("MotionDesign", "Motion Design Actions")
		, FToolMenuInsert(TEXT("ToolActions"), EToolMenuInsertType::Before));

	MotionDesignSection.AddSubMenu(TEXT("ApplyPreset")
		, LOCTEXT("ApplyPresetLabel", "Apply Preset")
		, LOCTEXT("ApplyPresetTooltip", "Apply a Preset to the Selected Sequences")
		, FNewToolMenuDelegate::CreateSP(this, &FAvaNavigationToolProvider::GeneratePresetMenu));
	MotionDesignSection.AddMenuEntry(AvaSequenceNavigatorCommands.SpawnSequencePlayer);
	MotionDesignSection.AddMenuEntry(AvaSequenceNavigatorCommands.ExportSequence);

	const FGenericCommands& GenericCommands = FGenericCommands::Get();
	FToolMenuSection& GenericSection = ExtendedMenu->FindOrAddSection(TEXT("GenericActions"), LOCTEXT("GenericActionsHeader", "Generic Actions"));

	GenericSection.AddMenuEntry(GenericCommands.Duplicate);
	GenericSection.AddMenuEntry(GenericCommands.Delete);
	GenericSection.AddMenuEntry(GenericCommands.Rename);
}

void FAvaNavigationToolProvider::RemoveToolToolBarExtension()
{
	if (UToolMenu* const Menu = UToolMenus::Get()->FindMenu(UE::SequenceNavigator::GetToolBarMenuName()))
	{
		Menu->RemoveSection(ToolbarSectionName);
	}
}

void FAvaNavigationToolProvider::RemoveToolItemContextMenuExtension()
{
	if (UToolMenu* const Menu = UToolMenus::Get()->FindMenu(UE::SequenceNavigator::GetItemContextMenuName()))
	{
		Menu->RemoveSection(ContextMenuSectionName);
	}
}

void FAvaNavigationToolProvider::ExtendToolItemDragDropOp(SequenceNavigator::FNavigationToolItemDragDropOp& InDragDropOp)
{
	InDragDropOp.AddDropHandler<SequenceNavigator::FNavigationToolAvaSequenceDropHandler>(WeakAvaSequencer);
}

void FAvaNavigationToolProvider::OnSequenceAdded(UAvaSequence* const InAvaSequence)
{
	using namespace SequenceNavigator;

	const TSharedPtr<IAvaSequencer> AvaSequencer = WeakAvaSequencer.Pin();
	if (!AvaSequencer.IsValid())
	{
		return;
	}

	const TSharedPtr<ISequencer> Sequencer = AvaSequencer->GetSequencerPtr();
	if (!Sequencer.IsValid())
	{
		return;
	}

	const TSharedPtr<INavigationTool> NavigationTool = FNavigationToolExtender::FindNavigationTool(Sequencer.ToSharedRef());
	if (!NavigationTool.IsValid())
	{
		return;
	}

	NavigationTool->RequestRefresh();
}

void FAvaNavigationToolProvider::GeneratePresetMenu(UToolMenu* const InToolMenu)
{
	const UAvaSequencerSettings* const SequencerSettings = GetDefault<UAvaSequencerSettings>();
	if (!SequencerSettings)
	{
		return;
	}

	const TSharedPtr<IAvaSequencer> AvaSequencer = WeakAvaSequencer.Pin();
	if (!AvaSequencer.IsValid())
	{
		return;
	}

	const TSharedRef<IAvaSequencer> AvaSequencerRef = AvaSequencer.ToSharedRef();

	// Default Presets
	const TConstArrayView<FAvaSequencePreset> DefaultPresets = SequencerSettings->GetDefaultSequencePresets();
	if (!DefaultPresets.IsEmpty())
	{
		FToolMenuSection& DefaultPresetSection = InToolMenu->FindOrAddSection(TEXT("DefaultPresets"), LOCTEXT("DefaultPresetsLabel", "Default Presets"));
		for (const FAvaSequencePreset& Preset : DefaultPresets)
		{
			DefaultPresetSection.AddMenuEntry(Preset.PresetName
				, FText::FromName(Preset.PresetName)
				, FText::FromName(Preset.PresetName)
				, FSlateIcon()
				, FToolUIActionChoice(FExecuteAction::CreateSP(this, &FAvaNavigationToolProvider::ApplyDefaultPresetToSelection, Preset.PresetName)));
		}
	}

	// Custom Presets
	const TSet<FAvaSequencePreset>& CustomPresets = SequencerSettings->GetCustomSequencePresets();
	if (!CustomPresets.IsEmpty())
	{
		FToolMenuSection& CustomPresetSection = InToolMenu->FindOrAddSection(TEXT("CustomPresets"), LOCTEXT("CustomPresetsLabel", "Custom Presets"));
		for (const FAvaSequencePreset& Preset : CustomPresets)
		{
			CustomPresetSection.AddMenuEntry(Preset.PresetName
				, FText::FromName(Preset.PresetName)
				, FText::FromName(Preset.PresetName)
				, FSlateIcon()
				, FToolUIActionChoice(FExecuteAction::CreateSP(this, &FAvaNavigationToolProvider::ApplyCustomPresetToSelection, Preset.PresetName)));
		}
	}

	// Settings
	FToolMenuSection& SettingsSection = InToolMenu->FindOrAddSection(TEXT("Settings"));
	SettingsSection.AddSeparator(NAME_None);
	SettingsSection.AddMenuEntry(TEXT("OpenSettings")
		, LOCTEXT("OpenSettingsLabel", "Open Settings")
		, LOCTEXT("OpenSettingsTooltip", "Opens the Settings to customize the sequence presets")
		, FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Settings")
		, FExecuteAction::CreateLambda([]
		{
			if (const UAvaSequencerSettings* const AvaSequencerSettings = GetDefault<UAvaSequencerSettings>())
			{
				ISettingsModule& SettingsModule = FModuleManager::LoadModuleChecked<ISettingsModule>(TEXT("Settings"));
				SettingsModule.ShowViewer(AvaSequencerSettings->GetContainerName(), AvaSequencerSettings->GetCategoryName(), AvaSequencerSettings->GetSectionName());
			}
		}));
}

TArray<Sequencer::TViewModelPtr<SequenceNavigator::FNavigationToolAvaSequence>> FAvaNavigationToolProvider::GetSelectedSequenceItems() const
{
	using namespace SequenceNavigator;
	using namespace Sequencer;

	const TSharedPtr<INavigationTool> Tool = GetNavigationTool();
	if (!Tool.IsValid())
	{
		return {};
	}

	const TArray<FNavigationToolViewModelWeakPtr> WeakSelectedItems = Tool->GetSelectedItems();
	if (WeakSelectedItems.IsEmpty())
	{
		return {};
	}

	TArray<TViewModelPtr<FNavigationToolAvaSequence>> SelectedSequenceItems;
	SelectedSequenceItems.Reserve(WeakSelectedItems.Num());

	for (const FNavigationToolViewModelWeakPtr& WeakItem : WeakSelectedItems)
	{
		if (const TViewModelPtr<FNavigationToolAvaSequence> AvaSequenceItem = WeakItem.ImplicitPin())
		{
			SelectedSequenceItems.Add(AvaSequenceItem);
		}
	}

	return SelectedSequenceItems;
}

TArray<UAvaSequence*> FAvaNavigationToolProvider::GetSelectedSequences() const
{
	using namespace UE::SequenceNavigator;
	using namespace UE::Sequencer;

	const TArray<TViewModelPtr<FNavigationToolAvaSequence>> SelectedItems = GetSelectedSequenceItems();
	if (SelectedItems.IsEmpty())
	{
		return {};
	}

	TArray<UAvaSequence*> SelectedAvaSequences;
	SelectedAvaSequences.Reserve(SelectedItems.Num());

	for (const TViewModelPtr<FNavigationToolAvaSequence>& SequenceItem : SelectedItems)
	{
		if (UAvaSequence* const AvaSequence = SequenceItem->GetAvaSequence())
		{
			SelectedAvaSequences.Add(AvaSequence);
		}
	}

	return SelectedAvaSequences;
}

bool FAvaNavigationToolProvider::CanRelabelSelection() const
{
	return CanEditOrPlaySelection(/*InMinNumSelected=*/1, /*InMaxNumSelected=*/1);
}

void FAvaNavigationToolProvider::RelabelSelection()
{
	using namespace UE::SequenceNavigator;
	using namespace UE::Sequencer;

	const TSharedPtr<IAvaSequencer> AvaSequencer = WeakAvaSequencer.Pin();
	if (!AvaSequencer.IsValid())
	{
		return;
	}

	const TSharedPtr<ISequencer> Sequencer = AvaSequencer->GetSequencerPtr();
	if (!Sequencer.IsValid())
	{
		return;
	}

	const TSharedPtr<INavigationTool> NavigationTool = FNavigationToolExtender::FindNavigationTool(Sequencer.ToSharedRef());
	if (!NavigationTool.IsValid())
	{
		return;
	}

	const TArray<FNavigationToolViewModelWeakPtr> WeakSelectedItems = NavigationTool->GetSelectedItems();
	if (WeakSelectedItems.Num() != 1)
	{
		return;
	}

	const FNavigationToolViewModelPtr FirstSelectedItem = WeakSelectedItems[0].Pin();
	if (!FirstSelectedItem.IsValid())
	{
		return;
	}

	const TViewModelPtr<FNavigationToolAvaSequence> AvaSequenceItem = FirstSelectedItem.ImplicitCast();
	if (!AvaSequenceItem)
	{
		return;
	}

	// @Todo: Implement BeginRename function in base classes
	//AvaSequenceItem->BeginRename();
}

bool FAvaNavigationToolProvider::CanAddSequenceToSelection() const
{
	return CanEditOrPlaySelection(/*InMinNumSelected=*/0, /*InMaxNumSelected=*/1);
}

void FAvaNavigationToolProvider::AddSequenceToSelection()
{
	using namespace UE::SequenceNavigator;
	using namespace UE::Sequencer;

	const TSharedPtr<IAvaSequencer> AvaSequencer = WeakAvaSequencer.Pin();
	if (!AvaSequencer.IsValid())
	{
		return;
	}

	const TSharedPtr<ISequencer> Sequencer = AvaSequencer->GetSequencerPtr();
	if (!Sequencer.IsValid())
	{
		return;
	}

	const TSharedPtr<INavigationTool> NavigationTool = FNavigationToolExtender::FindNavigationTool(Sequencer.ToSharedRef());
	if (!NavigationTool.IsValid())
	{
		return;
	}

	TViewModelPtr<FNavigationToolAvaSequence> ParentSequenceItem;
	UAvaSequence* ParentSequence = nullptr;

	const TArray<TViewModelPtr<FNavigationToolAvaSequence>> SelectedSequences = GetSelectedSequenceItems();
	if (SelectedSequences.Num() == 1)
	{
		ParentSequenceItem = SelectedSequences[0];
		ParentSequence = ParentSequenceItem->GetAvaSequence();
	}

	UAvaSequence* const NewSequence = AvaSequencer->AddSequence(ParentSequence);
	if (!NewSequence)
	{
		return;
	}

	const FNavigationToolViewModelPtr ActualParentItem = ParentSequenceItem.IsValid()
		? ParentSequenceItem : NavigationTool->GetTreeRoot().ImplicitPin();

	const FNavigationToolViewModelPtr NewItem = NavigationTool->FindOrAdd<FNavigationToolAvaSequence>(SharedThis(this)
		, ActualParentItem, NewSequence);

	FNavigationToolAddItemParams AddChildItemParams;
	AddChildItemParams.WeakItem = NewItem;
	AddChildItemParams.WeakRelativeItem = ActualParentItem.AsWeak();
	AddChildItemParams.RelativeDropZone = EItemDropZone::OntoItem;
	AddChildItemParams.Flags = ENavigationToolAddItemFlags::Select | ENavigationToolAddItemFlags::Transact;
	AddChildItemParams.SelectionFlags = ENavigationToolItemSelectionFlags::SignalSelectionChange
		| ENavigationToolItemSelectionFlags::ScrollIntoView;

	NavigationTool->EnqueueItemAction<FNavigationToolAddItem>(AddChildItemParams);
}

bool FAvaNavigationToolProvider::CanDuplicateSelection() const
{
	return CanEditOrPlaySelection();
}

void FAvaNavigationToolProvider::DuplicateSelection()
{
	const IAvaSequencerProvider* const SequencerProvider = GetSequencerProvider();
	if (!SequencerProvider)
	{
		return;
	}

	IAvaSequenceProvider* const SequenceProvider = SequencerProvider->GetSequenceProvider();
	if (!SequenceProvider)
	{
		return;
	}

	const TArray<UAvaSequence*> SelectedSequences = GetSelectedSequences();
	if (SelectedSequences.IsEmpty())
	{
		return;
	}

	UObject* const Outer = SequenceProvider->ToUObject();
	if (!Outer)
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("DuplicateSequenceTransaction", "Duplicate Sequence"));

	Outer->Modify();

	for (UAvaSequence* const TemplateSequence : SelectedSequences)
	{
		if (TemplateSequence)
		{
			UAvaSequence* const DupedSequence = DuplicateObject<UAvaSequence>(TemplateSequence, Outer);
			SequenceProvider->AddSequence(DupedSequence);
		}
	}
}

bool FAvaNavigationToolProvider::CanDeleteSelection() const
{
	return CanEditOrPlaySelection();
}

void FAvaNavigationToolProvider::DeleteSelection()
{
	const TSharedPtr<IAvaSequencer> AvaSequencer = WeakAvaSequencer.Pin();
	if (!AvaSequencer.IsValid())
	{
		return;
	}

	AvaSequencer->DeleteSequences(TSet<UAvaSequence*> { GetSelectedSequences() });
}

bool FAvaNavigationToolProvider::CanPlaySelection() const
{
	return CanEditOrPlaySelection();
}

void FAvaNavigationToolProvider::PlaySelection()
{
	IAvaSequencePlaybackObject* const PlaybackObject = GetSequencerPlaybackObject();
	if (!PlaybackObject)
	{
		return;
	}

	FAvaSequencePlayParams PlaySettings;
	PlaySettings.AdvancedSettings.bRestoreState = true;

	const TArray<UAvaSequence*> SelectedSequences = GetSelectedSequences();
	for (UAvaSequence* const Sequence : SelectedSequences)
	{
		if (Sequence)
		{
			PlaybackObject->PlaySequence(Sequence, PlaySettings);
		}
	}
}

bool FAvaNavigationToolProvider::CanContinueSelection() const
{
	return CanEditOrPlaySelection();
}

void FAvaNavigationToolProvider::ContinueSelection()
{
	IAvaSequencePlaybackObject* const PlaybackObject = GetSequencerPlaybackObject();
	if (!PlaybackObject)
	{
		return;
	}

	const TArray<UAvaSequence*> SelectedSequences = GetSelectedSequences();
	for (UAvaSequence* const Sequence : SelectedSequences)
	{
		if (Sequence)
		{
			PlaybackObject->ContinueSequence(Sequence);
		}
	}
}

bool FAvaNavigationToolProvider::CanStopSelection() const
{
	return CanEditOrPlaySelection();
}

void FAvaNavigationToolProvider::StopSelection()
{
	IAvaSequencePlaybackObject* const PlaybackObject = GetSequencerPlaybackObject();
	if (!PlaybackObject)
	{
		return;
	}

	const TArray<UAvaSequence*> SelectedSequences = GetSelectedSequences();
	for (UAvaSequence* const Sequence : SelectedSequences)
	{
		if (Sequence)
		{
			PlaybackObject->StopSequence(Sequence);
		}
	}
}

bool FAvaNavigationToolProvider::CanExportSelection() const
{
	return CanEditOrPlaySelection();
}

void FAvaNavigationToolProvider::ExportSelection()
{
	IAvaSequencerProvider* const SequencerProvider = const_cast<IAvaSequencerProvider*>(GetSequencerProvider());
	if (!SequencerProvider)
	{
		return;
	}

	const TArray<UAvaSequence*> SelectedSequences = GetSelectedSequences();
	if (SelectedSequences.IsEmpty())
	{
		return;
	}

	TArray<TWeakObjectPtr<>> WeakSelectedObjects;
	Algo::Transform(SelectedSequences, WeakSelectedObjects, [](UAvaSequence* const InSequence)
		{
			return InSequence;
		});

	SequencerProvider->ExportSequences(SelectedSequences);

	FNotificationInfo Info(LOCTEXT("ExportSuccess", "Sequence Exported Successfully!"));
	Info.bUseLargeFont = false;
	Info.bUseSuccessFailIcons = false;
	Info.ExpireDuration = 5.0f;
	Info.HyperlinkText = LOCTEXT("ShowNewAssetsInContentBrowser", "Show in content browser");
	Info.Hyperlink = FSimpleDelegate::CreateLambda([WeakSelectedObjects]()
		{
			TArray<UObject*> SyncObjects;
			SyncObjects.Reserve(WeakSelectedObjects.Num());

			for (const TWeakObjectPtr<> WeakObject : WeakSelectedObjects)
			{
				if (UObject* const SyncObject = WeakObject.Get())
				{
					SyncObjects.Add(SyncObject);
				}
			}

			FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
			ContentBrowserModule.Get().SyncBrowserToAssets(SyncObjects);
		});

	FSlateNotificationManager::Get().AddNotification(Info);
}

bool FAvaNavigationToolProvider::CanSpawnPlayersForSelection() const
{
	return CanEditOrPlaySelection();
}

void FAvaNavigationToolProvider::SpawnPlayersForSelection()
{
	if (!GEditor)
	{
		return;
	}

	IAvaSequencePlaybackObject* const PlaybackObject = GetSequencerPlaybackObject();
	if (!PlaybackObject)
	{
		return;
	}

	const UObject* const PlaybackContext = PlaybackObject->GetPlaybackContext();
	if (!PlaybackContext)
	{
		return;
	}

	UWorld* const World = PlaybackContext->GetWorld();
	if (!World)
	{
		return;
	}

	UActorFactory* const ActorFactory = GEditor->FindActorFactoryForActorClass(AAvaSequenceActor::StaticClass());
	if (!ActorFactory)
	{
		return;
	}

	const TArray<UAvaSequence*> Sequences = GetSelectedSequences();
	if (Sequences.IsEmpty())
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("SpawnSequencePlayers", "Spawn Sequence Players"));

	for (UAvaSequence* const Sequence : Sequences)
	{
		if (Sequence)
		{
			GEditor->UseActorFactory(ActorFactory, FAssetData(Sequence), &FTransform::Identity);
		}
	}

	const FText NotifyText = FText::Format(LOCTEXT("SpawnPlayerSuccess", "{0} Sequence Players Spawned Successfully!")
		, FText::AsNumber(Sequences.Num()));

	FNotificationInfo Info(NotifyText);
	Info.bUseLargeFont = false;
	Info.bUseSuccessFailIcons = false;
	FSlateNotificationManager::Get().AddNotification(Info);
}

void FAvaNavigationToolProvider::ApplyDefaultPresetToSelection(FName InPresetName)
{
	const UAvaSequencerSettings* SequencerSettings = GetDefault<UAvaSequencerSettings>();
	if (!SequencerSettings)
	{
		return;
	}

	const TConstArrayView<FAvaSequencePreset> DefaultSequencePresets = SequencerSettings->GetDefaultSequencePresets();

	const int32 PresetIndex = DefaultSequencePresets.Find(FAvaSequencePreset(InPresetName));
	if (PresetIndex == INDEX_NONE)
	{
		return;
	}

	ApplyPresetToSelection(DefaultSequencePresets[PresetIndex]);
}

void FAvaNavigationToolProvider::ApplyCustomPresetToSelection(FName InPresetName)
{
	const UAvaSequencerSettings* SequencerSettings = GetDefault<UAvaSequencerSettings>();
	if (!SequencerSettings)
	{
		return;
	}

	const FAvaSequencePreset* SequencePreset = SequencerSettings->GetCustomSequencePresets().Find(FAvaSequencePreset(InPresetName));
	if (!SequencePreset)
	{
		return;
	}

	ApplyPresetToSelection(*SequencePreset);
}

void FAvaNavigationToolProvider::ApplyPresetToSelection(const FAvaSequencePreset& InPreset)
{
	const TArray<UAvaSequence*> SelectedSequences = GetSelectedSequences();
	if (SelectedSequences.IsEmpty())
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("ApplySequencePreset", "Apply Sequence Preset"));

	for (UAvaSequence* const AvaSequence : SelectedSequences)
	{
		if (AvaSequence)
		{
			InPreset.ApplyPreset(AvaSequence);
		}
	}
}

} // namespace UE::AvaSequencer

#undef LOCTEXT_NAMESPACE
