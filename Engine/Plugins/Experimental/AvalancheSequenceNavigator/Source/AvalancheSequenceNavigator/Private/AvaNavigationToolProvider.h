// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/IDelegateInstance.h"
#include "NavigationToolDefines.h"
#include "Providers/NavigationToolProvider.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"

class FUICommandList;
class IAvaSceneInterface;
class IAvaSequencePlaybackObject;
class IAvaSequenceProvider;
class IAvaSequencerProvider;
class IAvaSequencer;
class UAvaSequence;
class UMovieSceneSequence;
class UObject;
struct FAvaSequencePreset;
struct FNavigationToolSaveState;

namespace UE::SequenceNavigator
{
	class FNavigationToolAvaSequence;
	class FNavigationToolItem;
	class FNavigationToolItemDragDropOp;
	class INavigationTool;
}

namespace UE::AvaSequencer
{

class FAvaNavigationToolProvider : public SequenceNavigator::FNavigationToolProvider
{
public:
	static const FName Identifier;

	static const FText MotionDesignColumnViewName;

	FAvaNavigationToolProvider(const TSharedRef<IAvaSequencer>& InAvaSequencer);
	virtual ~FAvaNavigationToolProvider() override = default;

	//~ Begin INavigationToolProvider

	virtual FName GetIdentifier() const override;

	virtual TSet<TSubclassOf<UMovieSceneSequence>> GetSupportedSequenceClasses() const override;

	virtual FText GetDefaultColumnView() const override;

	virtual FNavigationToolSaveState* GetSaveState(const SequenceNavigator::INavigationTool& InTool) const override;
	virtual void SetSaveState(const SequenceNavigator::INavigationTool& InTool, const FNavigationToolSaveState& InSaveState) const override;

	virtual void BindCommands(const TSharedRef<FUICommandList>& InCommandList) override;

	virtual void OnActivate() override;
	virtual void OnDeactivate() override;

	virtual void OnExtendColumns(SequenceNavigator::FNavigationToolColumnExtender& OutExtender) override;
	virtual void OnExtendColumnViews(TSet<FNavigationToolColumnView>& OutColumnViews) override;
	virtual void OnExtendItemChildren(SequenceNavigator::INavigationTool& InTool
		, const SequenceNavigator::FNavigationToolViewModelPtr& InParentItem
		, TArray<SequenceNavigator::FNavigationToolViewModelWeakPtr>& OutWeakChildren
		, const bool bInRecursive) override;
	virtual void OnExtendBuiltInFilters(TArray<FNavigationToolBuiltInFilterParams>& OutFilterParams) override;

	//~ End INavigationToolProvider

private:
	static const FName ToolbarSectionName;
	static const FName ContextMenuSectionName;

	TSharedPtr<SequenceNavigator::INavigationTool> GetNavigationTool() const;
	IAvaSceneInterface* GetSceneInterface(const SequenceNavigator::INavigationTool& InTool) const;
	IAvaSequenceProvider* GetSequenceProvider(const SequenceNavigator::INavigationTool& InTool) const;
	const IAvaSequencerProvider* GetSequencerProvider() const;
	IAvaSequencePlaybackObject* GetSequencerPlaybackObject() const;

	bool CanEditOrPlaySelection(const int32 InMinNumSelected = 1, const int32 InMaxNumSelected = INDEX_NONE) const;

	void ExtendToolToolBar();
	void ExtendToolItemContextMenu();

	void RemoveToolToolBarExtension();
	void RemoveToolItemContextMenuExtension();

	void ExtendToolItemDragDropOp(SequenceNavigator::FNavigationToolItemDragDropOp& InDragDropOp);

	void OnSequenceAdded(UAvaSequence* const InAvaSequence);

	void GeneratePresetMenu(UToolMenu* const InToolMenu);

	TArray<Sequencer::TViewModelPtr<SequenceNavigator::FNavigationToolAvaSequence>> GetSelectedSequenceItems() const;
	TArray<UAvaSequence*> GetSelectedSequences() const;

	bool CanRelabelSelection() const;
	void RelabelSelection();

	bool CanAddSequenceToSelection() const;
	void AddSequenceToSelection();

	bool CanDuplicateSelection() const;
	void DuplicateSelection();

	bool CanDeleteSelection() const;
	void DeleteSelection();

	bool CanPlaySelection() const;
	void PlaySelection();

	bool CanContinueSelection() const;
	void ContinueSelection();

	bool CanStopSelection() const;
	void StopSelection();

	bool CanExportSelection() const;
	void ExportSelection();

	bool CanSpawnPlayersForSelection() const;
	void SpawnPlayersForSelection();

	void ApplyDefaultPresetToSelection(const FName InPresetName);
	void ApplyCustomPresetToSelection(const FName InPresetName);
	void ApplyPresetToSelection(const FAvaSequencePreset& InPreset);

	TWeakPtr<IAvaSequencer> WeakAvaSequencer;

	TSharedRef<FUICommandList> ToolCommands;

	FDelegateHandle DragDropInitializedDelegate;
};

} // namespace UE::AvaSequencer
