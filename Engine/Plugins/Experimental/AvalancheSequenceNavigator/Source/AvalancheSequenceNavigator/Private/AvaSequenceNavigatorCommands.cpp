// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSequenceNavigatorCommands.h"

#define LOCTEXT_NAMESPACE "AvaSequenceNavigatorCommands"

void FAvaSequenceNavigatorCommands::RegisterCommands()
{
	UI_COMMAND(AddNew
		, "Add New"
		, "Adds a new embedded Sequence to the level"
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(PlaySelected
		, "Play"
		, "Plays all selected Sequences"
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(ContinueSelected
		, "Continue"
		, "Continues all selected Sequences"
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(StopSelected
		, "Stop"
		, "Stops all selected Sequences"
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(ExportSequence
		, "Export"
		, "Exports the Sequence into its own Asset"
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(SpawnSequencePlayer
		, "Spawn Sequence Player"
		, "Spawns a Sequence Player for the selected Sequence"
		, EUserInterfaceActionType::Button
		, FInputChord());
}

#undef LOCTEXT_NAMESPACE
