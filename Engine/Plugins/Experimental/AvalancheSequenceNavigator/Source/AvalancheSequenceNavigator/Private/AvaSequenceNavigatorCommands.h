// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "Styling/AppStyle.h"

class FAvaSequenceNavigatorCommands : public TCommands<FAvaSequenceNavigatorCommands>
{
public:
	FAvaSequenceNavigatorCommands()
		: TCommands<FAvaSequenceNavigatorCommands>(TEXT("AvaSequenceNavigatorCommands")
		, NSLOCTEXT("MotionDesignSequenceNavigatorCommands", "MotionDesignSequenceNavigatorCommands", "Motion Design Sequence Navigator Commands")
		, NAME_None
		, FAppStyle::GetAppStyleSetName())
	{
	}

	virtual void RegisterCommands() override;

	TSharedPtr<FUICommandInfo> AddNew;

	TSharedPtr<FUICommandInfo> PlaySelected;

	TSharedPtr<FUICommandInfo> ContinueSelected;

	TSharedPtr<FUICommandInfo> StopSelected;

	TSharedPtr<FUICommandInfo> ExportSequence;

	TSharedPtr<FUICommandInfo> SpawnSequencePlayer;
};
