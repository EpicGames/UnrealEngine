// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvalancheSequenceNavigatorModule.h"
#include "AvaNavigationToolProvider.h"
#include "AvaSequenceNavigatorCommands.h"
#include "AvaSequencerSubsystem.h"
#include "IAvaSequencer.h"
#include "ISequencer.h"
#include "ISequencerModule.h"
#include "Modules/ModuleManager.h"
#include "NavigationToolExtender.h"

#define LOCTEXT_NAMESPACE "AvalancheSequenceNavigatorModule"

using namespace UE::AvaSequencer;
using namespace UE::SequenceNavigator;

void FAvalancheSequenceNavigatorModule::StartupModule()
{
	FAvaSequenceNavigatorCommands::Register();

	if (ISequencerModule* const SequencerModule = FModuleManager::LoadModulePtr<ISequencerModule>(TEXT("Sequencer")))
	{
		SequencerCreatedHandle = SequencerModule->RegisterOnSequencerCreated(
			FOnSequencerCreated::FDelegate::CreateRaw(this, &FAvalancheSequenceNavigatorModule::OnSequencerCreated));
	}
}

void FAvalancheSequenceNavigatorModule::ShutdownModule()
{
	if (ISequencerModule* const SequencerModule = FModuleManager::LoadModulePtr<ISequencerModule>(TEXT("Sequencer")))
	{
		SequencerModule->UnregisterOnSequencerCreated(SequencerCreatedHandle);
		SequencerCreatedHandle.Reset();
	}

	FAvaSequenceNavigatorCommands::Unregister();
}

void FAvalancheSequenceNavigatorModule::OnSequencerCreated(const TSharedRef<ISequencer> InSequencer)
{
	SequencerClosedHandle = InSequencer->OnCloseEvent().AddRaw(this, &FAvalancheSequenceNavigatorModule::OnSequencerClosed);

	if (UObject* const PlaybackContextObject = InSequencer->GetPlaybackContext())
	{
		if (UWorld* const World = PlaybackContextObject->GetWorld())
		{
			if (UAvaSequencerSubsystem* const SequencerSubsystem = World->GetSubsystem<UAvaSequencerSubsystem>())
			{
				AvaSequencerCreatedHandle = SequencerSubsystem->OnSequencerCreated().AddRaw(this, &FAvalancheSequenceNavigatorModule::OnAvaSequencerCreated);
			}
		}
	}
}

void FAvalancheSequenceNavigatorModule::OnSequencerClosed(const TSharedRef<ISequencer> InSequencer)
{
	if (!NavigationToolProvider.IsValid())
	{
		return;
	}

	const FName ToolId = FNavigationToolExtender::GetToolInstanceId(*InSequencer);

	if (FNavigationToolExtender::UnregisterToolProvider(ToolId, NavigationToolProvider->GetIdentifier()))
	{
		NavigationToolProvider.Reset();
		SequencerClosedHandle.Reset();
	}
}

void FAvalancheSequenceNavigatorModule::OnAvaSequencerCreated(const TSharedRef<IAvaSequencer> InAvaSequencer)
{
	const TSharedPtr<ISequencer> Sequencer = InAvaSequencer->GetSequencerPtr();
	if (!Sequencer.IsValid())
	{
		return;
	}

	const FName ToolId = FNavigationToolExtender::GetToolInstanceId(*Sequencer);

	const TSharedPtr<FNavigationToolProvider> FoundProvider = FNavigationToolExtender::FindToolProvider(ToolId
		, FAvaNavigationToolProvider::Identifier);
	if (!FoundProvider.IsValid())
	{
		NavigationToolProvider = MakeShared<FAvaNavigationToolProvider>(InAvaSequencer);
		if (NavigationToolProvider->IsSequenceSupported(Sequencer->GetRootMovieSceneSequence()))
		{
			FNavigationToolExtender::RegisterToolProvider(Sequencer.ToSharedRef(), NavigationToolProvider.ToSharedRef());
		}
		else
		{
			NavigationToolProvider.Reset();
		}
	}
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FAvalancheSequenceNavigatorModule, AvalancheSequenceNavigator)
