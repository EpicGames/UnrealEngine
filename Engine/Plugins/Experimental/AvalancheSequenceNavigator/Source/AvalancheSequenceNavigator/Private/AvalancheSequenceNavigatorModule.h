// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/IDelegateInstance.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"

class IAvaSequencer;
class ISequencer;

namespace UE::AvaSequencer
{
	class FAvaNavigationToolProvider;
}

class FAvalancheSequenceNavigatorModule : public IModuleInterface
{
public:
	//~ Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface

private:
	void OnSequencerCreated(const TSharedRef<ISequencer> InSequencer);
	void OnSequencerClosed(const TSharedRef<ISequencer> InSequencer);

	void OnAvaSequencerCreated(const TSharedRef<IAvaSequencer> InAvaSequencer);

	FDelegateHandle SequencerCreatedHandle;
	FDelegateHandle SequencerClosedHandle;

	FDelegateHandle AvaSequencerCreatedHandle;

	/** The provider that supplies data and extends the Sequence Navigator */
	TSharedPtr<UE::AvaSequencer::FAvaNavigationToolProvider> NavigationToolProvider;
};
