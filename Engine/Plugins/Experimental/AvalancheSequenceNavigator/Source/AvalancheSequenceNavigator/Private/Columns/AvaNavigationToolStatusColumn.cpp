// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaNavigationToolStatusColumn.h"
#include "Columns/SAvaNavigationToolStatus.h"
#include "ISequencer.h"
#include "Items/NavigationToolAvaSequence.h"
#include "Styling/AppStyle.h"
#include "INavigationTool.h"
#include "INavigationToolView.h"

#define LOCTEXT_NAMESPACE "AvaNavigationToolStatusColumn"

namespace UE::SequenceNavigator
{

UE_SEQUENCER_DEFINE_CASTABLE(FAvaNavigationToolStatusColumn)

FText FAvaNavigationToolStatusColumn::GetColumnDisplayNameText() const
{
	return LOCTEXT("StatusColumn", "Status");
}

const FSlateBrush* FAvaNavigationToolStatusColumn::GetIconBrush() const
{
	return FAppStyle::GetBrush(TEXT("FoliageEditMode.BubbleBorder"));
}

SHeaderRow::FColumn::FArguments FAvaNavigationToolStatusColumn::ConstructHeaderRowColumn(const TSharedRef<INavigationToolView>& InToolView, const float InFillSize)
{
	const FName ColumnId = GetColumnId();

	return SHeaderRow::Column(ColumnId)
		.FillWidth(InFillSize)
		.HAlignHeader(HAlign_Center)
		.VAlignHeader(VAlign_Center)
		.HAlignCell(HAlign_Fill)
		.VAlignCell(VAlign_Center)
		.DefaultLabel(GetColumnDisplayNameText())
		.OnGetMenuContent(InToolView, &INavigationToolView::GetColumnMenuContent, ColumnId);
}

TSharedRef<SWidget> FAvaNavigationToolStatusColumn::ConstructRowWidget(const FNavigationToolViewModelPtr& InItem
	, const TSharedRef<INavigationToolView>& InView
	, const TSharedRef<SNavigationToolTreeRow>& InRow)
{
	if (!InItem.AsModel()->IsA<FNavigationToolAvaSequence>())
	{
		return SNullWidget::NullWidget;
	}

	const TSharedPtr<INavigationTool> OwnerTool = InView->GetOwnerTool();
	if (!OwnerTool.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	const TSharedPtr<ISequencer> Sequencer = OwnerTool->GetSequencer();
	if (!Sequencer.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	return SNew(SAvaNavigationToolStatus, InItem, InView, InRow, Sequencer.ToSharedRef());
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
