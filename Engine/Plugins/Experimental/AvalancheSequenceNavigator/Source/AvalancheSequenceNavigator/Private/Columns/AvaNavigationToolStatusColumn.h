// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Columns/NavigationToolColumn.h"
#include "MVVM/ICastable.h"

#define UE_API AVALANCHESEQUENCENAVIGATOR_API

namespace UE::SequenceNavigator
{

class FAvaNavigationToolStatusColumn
	: public FNavigationToolColumn
{
public:
	UE_SEQUENCER_DECLARE_CASTABLE_API(UE_API, FAvaNavigationToolStatusColumn
		, FNavigationToolColumn);

	static FName StaticColumnId() { return TEXT("Status"); }

protected:
	//~ Begin INavigationToolColumn
	virtual FName GetColumnId() const override { return StaticColumnId(); }
	virtual FText GetColumnDisplayNameText() const override;
	virtual const FSlateBrush* GetIconBrush() const override;
	virtual bool ShouldShowColumnByDefault() const override { return false; }
	virtual float GetFillWidth() const override { return 10.f; }
	virtual SHeaderRow::FColumn::FArguments ConstructHeaderRowColumn(const TSharedRef<INavigationToolView>& InToolView, const float InFillSize) override;
	virtual TSharedRef<SWidget> ConstructRowWidget(const FNavigationToolViewModelPtr& InItem
		, const TSharedRef<INavigationToolView>& InView
		, const TSharedRef<SNavigationToolTreeRow>& InRow) override;
	//~ End INavigationToolColumn
};

} // namespace UE::SequenceNavigator

#undef UE_API
