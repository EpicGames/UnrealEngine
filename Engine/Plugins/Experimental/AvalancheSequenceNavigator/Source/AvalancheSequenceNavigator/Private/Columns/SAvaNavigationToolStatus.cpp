// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaNavigationToolStatus.h"
#include "AvaSequence.h"
#include "AvaSequencePlayer.h"
#include "AvaSequencerUtils.h"
#include "IAvaSceneInterface.h"
#include "IAvaSequencerProvider.h"
#include "INavigationToolView.h"
#include "ISequencer.h"
#include "Items/NavigationToolAvaSequence.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Notifications/SProgressBar.h"
#include "Widgets/SNavigationToolTreeRow.h"

#define LOCTEXT_NAMESPACE "SAvaNavigationToolStatus"

namespace UE::SequenceNavigator
{

using namespace UE::Sequencer;

void SAvaNavigationToolStatus::Construct(const FArguments& InArgs
	, const FNavigationToolViewModelPtr& InItem
	, const TSharedRef<INavigationToolView>& InView
	, const TSharedRef<SNavigationToolTreeRow>& InRowWidget
	, const TSharedRef<ISequencer>& InSequencer)
{
	WeakItem = InItem;
	WeakView = InView;
	WeakRowWidget = InRowWidget;
	WeakSequencer = InSequencer;

	const TViewModelPtr<FNavigationToolAvaSequence> AvaSequenceItem = InItem.ImplicitCast();
	if (!AvaSequenceItem.IsValid())
	{
		return;
	}

	ChildSlot
	[
		SNew(SBox)
		.Padding(5.f, 2.f)
		[
			SNew(SOverlay)
			+ SOverlay::Slot()
			.Padding(1.f, 1.f)
			[
				SNew(SProgressBar)
				.Percent(this, &SAvaNavigationToolStatus::GetProgressPercent)
				.Visibility(EVisibility::Visible)
			]
			+ SOverlay::Slot()
			.Padding(1.f, 0.f)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(this, &SAvaNavigationToolStatus::GetProgressText)
				.TextStyle(&FAppStyle::GetWidgetStyle<FTextBlockStyle>(TEXT("SmallText")))
				.Justification(ETextJustify::Type::Center)
			]
		]
	];
}

void SAvaNavigationToolStatus::Tick(const FGeometry& InAllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return;
	}

	const FNavigationToolViewModelPtr Item = WeakItem.Pin();
	if (!Item.IsValid())
	{
		return;
	}

	const TViewModelPtr<FNavigationToolAvaSequence> AvaSequenceItem = Item.ImplicitCast();
	if (!AvaSequenceItem.IsValid())
	{
		return;
	}

	UAvaSequence* const Sequence = AvaSequenceItem->GetAvaSequence();
	if (!Sequence)
	{
		return;
	}

	IAvaSceneInterface* const SceneInterface = FAvaSequencerUtils::GetSceneInterface(Sequencer.ToSharedRef());
	if (!SceneInterface)
	{
		return;
	}

	IAvaSequencePlaybackObject* const PlaybackObject = SceneInterface->GetPlaybackObject();
	if (!PlaybackObject)
	{
		return;
	}

	UAvaSequencePlayer* const Player = PlaybackObject->GetSequencePlayer(Sequence);

	bSequenceInProgress = Player != nullptr;
	if (bSequenceInProgress)
	{
		const EMovieScenePlayerStatus::Type Status = Player->GetPlaybackStatus();
		CurrentFrame = Player->GetCurrentTime().ConvertTo(Player->GetDisplayRate());
		TotalFrames = Player->GetDuration().ConvertTo(Player->GetDisplayRate());

		FFormatNamedArguments Args;
		if (Status == EMovieScenePlayerStatus::Playing)
		{
			Args.Add("Status", LOCTEXT("SequenceStatus_Playing", "Playing"));
		}
		else
		{
			Args.Add("Status", LOCTEXT("SequenceStatus_Stopped" , "Stopped"));
		}
		Args.Add("Current", FText::AsNumber(CurrentFrame.GetFrame().Value));
		Args.Add("Total", FText::AsNumber(TotalFrames.GetFrame().Value));

		StatusText =  FText::Format(LOCTEXT("SequenceStatus_Text", "{Status} ({Current} / {Total})"), Args);

		Progress = FMath::IsNearlyZero(TotalFrames.AsDecimal())
			? 0.f : (CurrentFrame.AsDecimal() / TotalFrames.AsDecimal());
	}
	else
	{
		StatusText = LOCTEXT("SequenceStatus_Unknown", "Not Playing");
		Progress = 0.f;
	}
}

FText SAvaNavigationToolStatus::GetProgressText() const
{
	return StatusText;
}

TOptional<float> SAvaNavigationToolStatus::GetProgressPercent() const
{
	return Progress;
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
