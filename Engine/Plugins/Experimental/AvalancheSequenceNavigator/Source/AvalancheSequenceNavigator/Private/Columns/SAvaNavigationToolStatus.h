// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/FrameTime.h"
#include "NavigationToolDefines.h"
#include "Widgets/SCompoundWidget.h"

class FText;
class ISequencer;

namespace UE::SequenceNavigator
{

class INavigationToolView;
class SNavigationToolTreeRow;

class SAvaNavigationToolStatus : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAvaNavigationToolStatus) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs
		, const FNavigationToolViewModelPtr& InItem
		, const TSharedRef<INavigationToolView>& InView
		, const TSharedRef<SNavigationToolTreeRow>& InRowWidget
		, const TSharedRef<ISequencer>& InSequencer);

private:
	//~ Begin SWidget
	virtual void Tick(const FGeometry& InAllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	//~ End SWidget

	FText GetProgressText() const;

	TOptional<float> GetProgressPercent() const;

	FNavigationToolViewModelWeakPtr WeakItem;
	TWeakPtr<INavigationToolView> WeakView;
	TWeakPtr<SNavigationToolTreeRow> WeakRowWidget;
	TWeakPtr<ISequencer> WeakSequencer;

	FText StatusText;
	FFrameTime CurrentFrame;
	FFrameTime TotalFrames;
	float Progress = 0.f;
	bool bSequenceInProgress = false;
};

} // namespace UE::SequenceNavigator
