// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavigationToolAvaSequenceDropHandler.h"
#include "AvaSequence.h"
#include "IAvaSequenceProvider.h"
#include "IAvaSequencer.h"
#include "IAvaSequencerProvider.h"
#include "INavigationTool.h"
#include "ItemActions/NavigationToolAddItem.h"
#include "ItemActions/NavigationToolRemoveItem.h"
#include "Items/NavigationToolAvaSequence.h"
#include "Misc/Optional.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "NavigationToolAvaSequenceDropHandler"

namespace UE::SequenceNavigator
{
	
using namespace UE::Sequencer;

UE_SEQUENCER_DEFINE_CASTABLE(FNavigationToolAvaSequenceDropHandler);

FNavigationToolAvaSequenceDropHandler::FNavigationToolAvaSequenceDropHandler(const TWeakPtr<IAvaSequencer>& InWeakAvaSequencer)
	: WeakAvaSequencer(InWeakAvaSequencer)
{
}

bool FNavigationToolAvaSequenceDropHandler::IsDraggedItemSupported(const FNavigationToolViewModelPtr& InDraggedItem) const
{
	return InDraggedItem.AsModel()->IsA<FNavigationToolAvaSequence>();
}

TOptional<EItemDropZone> FNavigationToolAvaSequenceDropHandler::CanDrop(const EItemDropZone InDropZone
	, const FNavigationToolViewModelPtr& InTargetItem) const
{
	const bool bTargetIsRoot = (InTargetItem->GetItemId() == FNavigationToolItemId::RootId);

	UAvaSequence* TargetAvaSequence = nullptr;

	if (!bTargetIsRoot)
	{
		if (FNavigationToolAvaSequence* const TargetAvaSequenceItem = InTargetItem.AsModel()->CastThis<FNavigationToolAvaSequence>())
		{
			TargetAvaSequence = TargetAvaSequenceItem->GetAvaSequence();
		}
		if (!TargetAvaSequence)
		{
			return TOptional<EItemDropZone>();
		}
	}

	for (const FNavigationToolViewModelWeakPtr& WeakItem : WeakItems)
	{
		const FNavigationToolViewModelPtr Item = WeakItem.Pin();
		if (!Item.IsValid())
		{
			continue;
		}

		// Only allow moving sequences to other Motion Design sequences
		const FNavigationToolAvaSequence* const AvaSequenceItemToMove = Item.AsModel()->CastThis<FNavigationToolAvaSequence>();
		if (!AvaSequenceItemToMove)
		{
			return TOptional<EItemDropZone>();
		}

		// Only allow moving valid sequences and sequences that aren't the target
		UAvaSequence* const AvaSequenceToMove = AvaSequenceItemToMove->GetAvaSequence();
		if (!AvaSequenceToMove || AvaSequenceToMove == TargetAvaSequence)
		{
			return TOptional<EItemDropZone>();
		}

		if (!bTargetIsRoot && TargetAvaSequence)
		{
			if (TargetAvaSequence->GetChildren().Contains(AvaSequenceToMove))
			{
				return TOptional<EItemDropZone>();
			}
		}
	}

	switch (ActionType)
	{
		case ENavigationToolDragDropActionType::Move:
			// Make sure the destination is not one of the items we're moving
			if (!WeakItems.Contains(InTargetItem))
			{
				return InDropZone;
			}
			break;

		case ENavigationToolDragDropActionType::Copy:
			return InDropZone;
	}

	return TOptional<EItemDropZone>();
}

bool FNavigationToolAvaSequenceDropHandler::Drop(const EItemDropZone InDropZone, const FNavigationToolViewModelPtr& InTargetItem)
{
	switch (ActionType)
	{
	case ENavigationToolDragDropActionType::Move:
		MoveItems(InDropZone, InTargetItem);
		break;

	case ENavigationToolDragDropActionType::Copy:
		DuplicateItems(WeakItems, InTargetItem, InDropZone);
		break;

	default:
		return false;
	}

	return true;
}

void FNavigationToolAvaSequenceDropHandler::MoveItems(const EItemDropZone InDropZone, const FNavigationToolViewModelPtr& InTargetItem)
{
	using namespace UE::Sequencer;

	TArray<TSharedPtr<INavigationToolItemAction>> ItemActions;

	FNavigationToolAddItemParams AddParams;
	AddParams.WeakRelativeItem = InTargetItem;
	AddParams.RelativeDropZone = InDropZone;
	AddParams.Flags = ENavigationToolAddItemFlags::Select | ENavigationToolAddItemFlags::Transact;
	AddParams.SelectionFlags = ENavigationToolItemSelectionFlags::AppendToCurrentSelection;

	const TSet<FNavigationToolViewModelWeakPtr> WeakDraggedItemSet(WeakItems);

	// Remove all Items whose Parent is in the Item Set 
	WeakItems.RemoveAll([&WeakDraggedItemSet](const FNavigationToolViewModelWeakPtr& InWeakItem)
		{
			if (const FNavigationToolViewModelPtr Item = InWeakItem.Pin())
			{
				const FNavigationToolViewModelPtr ItemParent = Item->GetParent();
				return !ItemParent.IsValid()
					|| WeakDraggedItemSet.Contains(ItemParent);
			}
			return false;
		});

	// Reverse order for onto since Item->AddChild(...) adds it to Index 0, so last item would be at the top, which is reversed
	if (InDropZone == EItemDropZone::OntoItem)
	{
		Algo::Reverse(WeakItems);
	}

	for (const FNavigationToolViewModelWeakPtr& WeakItem : WeakItems)
	{
		if (const FNavigationToolViewModelPtr Item = WeakItem.Pin())
		{
			// Only allow moving sequences to other Motion Design sequences
			if (const TViewModelPtr<FNavigationToolAvaSequence> AvaSequenceItemToMove = Item.ImplicitCast())
			{
				AddParams.WeakItem = AvaSequenceItemToMove;
				ItemActions.Add(MakeShared<FNavigationToolAddItem>(AddParams));
			}
		}
	}

	InTargetItem->GetOwnerTool().EnqueueItemActions(MoveTemp(ItemActions));
}

void FNavigationToolAvaSequenceDropHandler::DuplicateItems(const TArray<FNavigationToolViewModelWeakPtr>& InWeakItems
	, const FNavigationToolViewModelWeakPtr& InWeakRelativeItem
	, const TOptional<EItemDropZone>& InRelativeDropZone)
{
	using namespace UE::Sequencer;

	const TSharedPtr<IAvaSequencer> AvaSequencer = WeakAvaSequencer.Pin();
	if (!AvaSequencer.IsValid())
	{
		return;
	}

	const IAvaSequencerProvider& SequencerProvider = AvaSequencer->GetProvider();

	IAvaSequenceProvider* const SequenceProvider = SequencerProvider.GetSequenceProvider();
	if (!SequenceProvider)
	{
		return;
	}

	UObject* const Outer = SequenceProvider->ToUObject();
	if (!Outer)
	{
		return;
	}

	// Gather list of sequences to duplicate
	TSet<UAvaSequence*> SequencesToDuplicate;

	for (const FNavigationToolViewModelWeakPtr& WeakItem : InWeakItems)
	{
		const FNavigationToolViewModelPtr Item = WeakItem.Pin();
		if (!Item.IsValid())
		{
			continue;
		}

		const TViewModelPtr<FNavigationToolAvaSequence> AvaSequenceItem = Item.ImplicitCast();
		if (!AvaSequenceItem.IsValid())
		{
			continue;
		}

		UAvaSequence* const TemplateSequence = AvaSequenceItem->GetAvaSequence();
		if (!TemplateSequence)
		{
			continue;
		}

		SequencesToDuplicate.Add(TemplateSequence);
	}

	if (SequencesToDuplicate.IsEmpty())
	{
		return;
	}

	// Duplicate sequence objects
	const FScopedTransaction Transaction(LOCTEXT("DuplicateSequencesTransaction", "Duplicate Sequence(s)"));

	Outer->Modify();

	for (const UAvaSequence* const TemplateSequence : SequencesToDuplicate)
	{
		if (UAvaSequence* const DupedSequence = DuplicateObject<UAvaSequence>(TemplateSequence, Outer))
		{
			SequenceProvider->AddSequence(DupedSequence);
		}
	}
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
