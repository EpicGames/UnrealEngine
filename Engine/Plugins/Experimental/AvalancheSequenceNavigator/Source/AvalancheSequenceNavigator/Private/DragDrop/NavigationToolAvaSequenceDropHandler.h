// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "DragDropOps/Handlers/NavigationToolItemDropHandler.h"

#define UE_API AVALANCHESEQUENCENAVIGATOR_API

class IAvaSequencer;
enum class EItemDropZone;
template<typename OptionalType> struct TOptional;

namespace UE::SequenceNavigator
{

class FNavigationToolAvaSequenceDropHandler
	: public FNavigationToolItemDropHandler
{
public:
	UE_SEQUENCER_DECLARE_CASTABLE_API(UE_API, FNavigationToolAvaSequenceDropHandler
		, FNavigationToolItemDropHandler);

	FNavigationToolAvaSequenceDropHandler(const TWeakPtr<IAvaSequencer>& InWeakAvaSequencer);

protected:
	//~ Begin FNavigationToolItemDropHandler
	virtual bool IsDraggedItemSupported(const FNavigationToolViewModelPtr& InDraggedItem) const override;
	virtual TOptional<EItemDropZone> CanDrop(const EItemDropZone InDropZone
		, const FNavigationToolViewModelPtr& InTargetItem) const override;
	virtual bool Drop(const EItemDropZone InDropZone, const FNavigationToolViewModelPtr& InTargetItem) override;
	//~ End FNavigationToolItemDropHandler

	void MoveItems(const EItemDropZone InDropZone, const FNavigationToolViewModelPtr& InTargetItem);

	void DuplicateItems(const TArray<FNavigationToolViewModelWeakPtr>& InWeakItems
		, const FNavigationToolViewModelWeakPtr& InWeakRelativeItem
		, const TOptional<EItemDropZone>& InRelativeDropZone);

	TWeakPtr<IAvaSequencer> WeakAvaSequencer;
};

} // namespace UE::SequenceNavigator

#undef UE_API
