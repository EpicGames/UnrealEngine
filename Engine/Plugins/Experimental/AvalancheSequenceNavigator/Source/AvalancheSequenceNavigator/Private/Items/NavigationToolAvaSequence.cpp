// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavigationToolAvaSequence.h"
#include "AvaEditorCoreStyle.h"
#include "AvaSequence.h"
#include "AvaSequencerUtils.h"
#include "IAvaSequencer.h"
#include "INavigationTool.h"
#include "ISequencer.h"
#include "Items/NavigationToolItemParameters.h"
#include "Items/NavigationToolSequence.h"
#include "ScopedTransaction.h"
#include "Styling/StyleColors.h"

#define LOCTEXT_NAMESPACE "NavigationToolAvaSequence"

namespace UE::SequenceNavigator
{

using namespace UE::Sequencer;

UE_SEQUENCER_DEFINE_CASTABLE(FNavigationToolAvaSequence)

FNavigationToolAvaSequence::FNavigationToolAvaSequence(INavigationTool& InTool
	, const FNavigationToolViewModelPtr& InParentItem
	, UAvaSequence* const InAvaSequence)
	: FNavigationToolSequence(InTool
	, InParentItem
	, InAvaSequence
	, nullptr
	, 0)
{
}

bool FNavigationToolAvaSequence::AddChild(const FNavigationToolAddItemParams& InAddItemParams)
{
	const TViewModelPtr<FNavigationToolAvaSequence> AvaSequenceItemToAdd = InAddItemParams.WeakItem.ImplicitPin();
	if (!AvaSequenceItemToAdd.IsValid())
	{
		return false;
	}

	if (!CanAddChild(AvaSequenceItemToAdd))
	{
		return false;
	}

	AddChildChecked(InAddItemParams);

	UAvaSequence* const ParentAvaSequence = GetAvaSequence();
	UAvaSequence* const AvaSequenceToAdd = AvaSequenceItemToAdd->GetAvaSequence();

	const TArray<TWeakObjectPtr<UAvaSequence>> TargetChildren = ParentAvaSequence->GetChildren();
	if (TargetChildren.Contains(AvaSequenceToAdd))
	{
		return false;
	}

	// Remove the sequence from any parent it's a child of
	if (UAvaSequence* const AvaSequenceToMoveParent = AvaSequenceToAdd->GetParent())
	{
		AvaSequenceToMoveParent->RemoveChild(AvaSequenceToAdd);
	}

	// Process moving sequence if it's parent is the target
	if (ParentAvaSequence == AvaSequenceToAdd->GetParent())
	{
		ParentAvaSequence->RemoveChild(AvaSequenceToAdd);

		if (UAvaSequence* const TargetParent = ParentAvaSequence->GetParent())
		{
			TargetParent->AddChild(AvaSequenceToAdd);
		}

		return true;
	}

	// Check if the Target Item is a Child of Sequence Item (to prevent cycle)
	UAvaSequence* CurrentItem = ParentAvaSequence;
	while (CurrentItem)
	{
		if (CurrentItem->GetParent() == AvaSequenceToAdd)
		{
			AvaSequenceToAdd->RemoveChild(CurrentItem);
			CurrentItem->SetParent(AvaSequenceToAdd->GetParent());

			break;
		}

		CurrentItem = CurrentItem->GetParent();
	}

	ParentAvaSequence->AddChild(AvaSequenceToAdd);

	return true;
}

bool FNavigationToolAvaSequence::RemoveChild(const FNavigationToolRemoveItemParams& InRemoveItemParams)
{
	const FNavigationToolViewModelPtr ItemToRemove = InRemoveItemParams.WeakItem.Pin();
	if (!ItemToRemove.IsValid())
	{
		return false;
	}

	// Check that we're removing a child item that is directly under us, and not a grandchild
	if (!WeakChildren.Contains(InRemoveItemParams.WeakItem))
	{
		return false;
	}

	if (const TViewModelPtr<FNavigationToolAvaSequence> AvaSequenceItemToRemove = ItemToRemove.ImplicitCast())
	{
		if (UAvaSequence* const AvaSequenceToRemove = AvaSequenceItemToRemove->GetAvaSequence())
		{
			// Remove the sequence from any parent it's a child of
			if (UAvaSequence* const AvaSequenceToRemoveParent = AvaSequenceToRemove->GetParent())
			{
				AvaSequenceToRemoveParent->RemoveChild(AvaSequenceToRemove);
			}
		}
	}

	return RemoveChildChecked(ItemToRemove);
}

void FNavigationToolAvaSequence::FindChildren(TArray<FNavigationToolViewModelWeakPtr>& OutWeakChildren, const bool bInRecursive)
{
	FNavigationToolSequence::FindChildren(OutWeakChildren, bInRecursive);

	UAvaSequence* const AvaSequence = GetAvaSequence();
	if (!AvaSequence)
	{
		return;
	}

	const TSharedPtr<ISequencer> Sequencer = Tool.GetSequencer();
	if (!Sequencer.IsValid())
	{
		return;
	}

	const TSharedPtr<FNavigationToolProvider> Provider = GetProvider();
	if (!Provider.IsValid())
	{
		return;
	}

	TArray<TWeakObjectPtr<UAvaSequence>> ChildSequences = AvaSequence->GetChildren();
	ChildSequences.RemoveAll([](const TWeakObjectPtr<UAvaSequence>& InChild)
		{
			return !InChild.IsValid();
		});
	ChildSequences.Sort([](const TWeakObjectPtr<UAvaSequence>& InA, const TWeakObjectPtr<UAvaSequence>& InB)
		{
			return InA->GetDisplayName().CompareTo(InB->GetDisplayName()) > 0;
		});

	const TSharedRef<FNavigationToolProvider> ProviderRef = Provider.ToSharedRef();
	const FNavigationToolViewModelPtr ParentItem = AsItemViewModel();

	for (const TWeakObjectPtr<UAvaSequence>& SequenceWeak : ChildSequences)
	{
		const FNavigationToolViewModelPtr NewItem = Tool.FindOrAdd<FNavigationToolAvaSequence>(ProviderRef
			, ParentItem, SequenceWeak.Get());
		OutWeakChildren.Add(NewItem);
		if (bInRecursive)
		{
			NewItem->FindChildren(OutWeakChildren, bInRecursive);
		}
	}
}

void FNavigationToolAvaSequence::GetItemProxies(TArray<TSharedPtr<FNavigationToolItemProxy>>& OutItemProxies)
{
	FNavigationToolSequence::GetItemProxies(OutItemProxies);
}

bool FNavigationToolAvaSequence::CanRename() const
{
	return GetAvaSequence() != nullptr;
}

void FNavigationToolAvaSequence::Rename(const FText& InNewName)
{
	UAvaSequence* const UnderlyingSequence = GetAvaSequence();
	if (!UnderlyingSequence)
	{
		return;
	}

	const FString NewNameString = InNewName.ToString();

	if (NewNameString.Equals(UnderlyingSequence->GetLabel().ToString(), ESearchCase::CaseSensitive))
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("AvaSequenceRename", "Rename Sequence"));

	UnderlyingSequence->Modify();
	UnderlyingSequence->SetLabel(FName(NewNameString));

	Tool.NotifyToolItemRenamed(SharedThis(this));
}

bool FNavigationToolAvaSequence::CanDelete() const
{
	return GetAvaSequence() != nullptr;
}

bool FNavigationToolAvaSequence::Delete()
{
	const TSharedPtr<ISequencer> Sequencer = Tool.GetSequencer();
	if (!Sequencer.IsValid())
	{
		return false;
	}

	UAvaSequence* const AvaSequence = GetAvaSequence();
	if (!AvaSequence)
	{
		return false;
	}

	const TSharedPtr<IAvaSequencer> AvaSequencer = FAvaSequencerUtils::GetAvaSequencer(Sequencer.ToSharedRef());
	if (!AvaSequencer.IsValid())
	{
		return false;
	}

	AvaSequencer->DeleteSequences({ AvaSequence });

	return true;
}

FText FNavigationToolAvaSequence::GetDisplayName() const
{
	if (const UAvaSequence* const AvaSequence = GetAvaSequence())
	{
		return AvaSequence->GetDisplayName();
	}
	return FText::GetEmpty();
}

FSlateIcon FNavigationToolAvaSequence::GetIcon() const
{
	return FSlateIcon(FAvaEditorCoreStyle::Get().GetStyleSetName(), TEXT("Icons.MotionDesign"));
}

FSlateColor FNavigationToolAvaSequence::GetIconColor() const
{
	const TSharedPtr<ISequencer> Sequencer = Tool.GetSequencer();
	if (!Sequencer.IsValid())
	{
		return FStyleColors::Foreground;
	}

	const UMovieSceneSequence* const RootSequence = Sequencer->GetRootMovieSceneSequence();
	if (!RootSequence)
	{
		return FStyleColors::Foreground;
	}

	if (GetAvaSequence() == RootSequence)
	{
		return FStyleColors::AccentGreen;
	}

	return FStyleColors::Foreground;
}

void FNavigationToolAvaSequence::OnSelect()
{
	FNavigationToolSequence::OnSelect();
}

void FNavigationToolAvaSequence::OnDoubleClick()
{
	UAvaSequence* const AvaSequence = GetAvaSequence();
	if (!AvaSequence)
	{
		return;
	}

	const TSharedPtr<ISequencer> Sequencer = Tool.GetSequencer();
	if (!Sequencer.IsValid())
	{
		return;
	}

	const TSharedPtr<IAvaSequencer> AvaSequencer = FAvaSequencerUtils::GetAvaSequencer(Sequencer.ToSharedRef());
	if (!AvaSequencer.IsValid())
	{
		return;
	}

	AvaSequencer->SetViewedSequence(AvaSequence);
}

UAvaSequence* FNavigationToolAvaSequence::GetAvaSequence() const
{
	return Cast<UAvaSequence>(GetSequence());
}

FNavigationToolItemId FNavigationToolAvaSequence::CalculateItemId() const
{
	return FNavigationToolItemId(GetParent(), GetAvaSequence());
}

TOptional<FColor> FNavigationToolAvaSequence::GetColor() const
{
	return Tool.FindItemColor(SharedThis(const_cast<FNavigationToolAvaSequence*>(this)));
}

void FNavigationToolAvaSequence::SetColor(const TOptional<FColor>& InColor)
{
	Tool.SetItemColor(SharedThis(this), InColor.Get(FColor()));
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
