// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Items/NavigationToolSequence.h"

class UAvaSequence;

namespace UE::SequenceNavigator
{

/**
 * Item in Navigation Tool representing a Motion Design Sequence.
 */
class FNavigationToolAvaSequence
	: public FNavigationToolSequence
{
public:
	UE_SEQUENCER_DECLARE_CASTABLE(FNavigationToolAvaSequence
		, FNavigationToolSequence)

	FNavigationToolAvaSequence(INavigationTool& InTool
		, const FNavigationToolViewModelPtr& InParentItem
		, UAvaSequence* const InAvaSequence);

	//~ Begin INavigationToolItem

	virtual bool AddChild(const FNavigationToolAddItemParams& InAddItemParams) override;
	virtual bool RemoveChild(const FNavigationToolRemoveItemParams& InRemoveItemParams) override;
	virtual void FindChildren(TArray<FNavigationToolViewModelWeakPtr>& OutWeakChildren, const bool bInRecursive) override;
	virtual void GetItemProxies(TArray<TSharedPtr<FNavigationToolItemProxy>>& OutItemProxies) override;

	virtual bool ShouldSort() const override { return true; }
	virtual bool CanBeTopLevel() const override { return true; }

	virtual bool CanRename() const override;
	virtual void Rename(const FText& InNewName) override;

	virtual bool CanDelete() const override;
	virtual bool Delete() override;

	virtual FText GetDisplayName() const override;
	virtual FSlateIcon GetIcon() const override;
	virtual FSlateColor GetIconColor() const override;

	virtual void OnSelect() override;
	virtual void OnDoubleClick() override;

	//~ End INavigationToolItem

	//~ Begin IColorExtension
	virtual TOptional<FColor> GetColor() const override;
	virtual void SetColor(const TOptional<FColor>& InColor) override;
	//~ End IColorExtension

	UAvaSequence* GetAvaSequence() const;

protected:
	//~Begin FNavigationToolItem
	virtual FNavigationToolItemId CalculateItemId() const override;
	//~End FNavigationToolItem
};

} // namespace UE::SequenceNavigator
